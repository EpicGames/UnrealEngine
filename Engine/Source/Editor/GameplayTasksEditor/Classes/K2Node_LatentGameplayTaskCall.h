// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "GameplayTask.h"
#include "HAL/PlatformCrt.h"
#include "K2Node_BaseAsyncTask.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "K2Node_LatentGameplayTaskCall.generated.h"

#define UE_API GAMEPLAYTASKSEDITOR_API

class FBlueprintActionDatabaseRegistrar;
class FName;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UEdGraphSchema;
class UEdGraphSchema_K2;
class UGameplayTask;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_LatentGameplayTaskCall : public UK2Node_BaseAsyncTask
{
	GENERATED_BODY()

public:
	UE_API UK2Node_LatentGameplayTaskCall(const FObjectInitializer& ObjectInitializer);

	// UEdGraphNode interface
	UE_API virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	// End of UEdGraphNode interface

	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	UE_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	UE_API bool ConnectSpawnProperties(UClass* ClassToSpawn, const UEdGraphSchema_K2* Schema, class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UEdGraphPin*& LastThenPin, UEdGraphPin* SpawnedActorReturnPin);		//Helper
	UE_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;

	UE_API void CreatePinsForClass(UClass* InClass);
	UE_API UEdGraphPin* GetClassPin(const TArray<UEdGraphPin*>* InPinsToSearch = NULL) const;
	UE_API UClass* GetClassToSpawn(const TArray<UEdGraphPin*>* InPinsToSearch = NULL) const;
	UE_API UEdGraphPin* GetResultPin() const;
	UE_API bool IsSpawnVarPin(UEdGraphPin* Pin);
	UE_API bool ValidateActorSpawning(class FKismetCompilerContext& CompilerContext, bool bGenerateErrors);
	UE_API bool ValidateActorArraySpawning(class FKismetCompilerContext& CompilerContext, bool bGenerateErrors);

	UPROPERTY()
	TArray<FName> SpawnParamPins;

	static UE_API void RegisterSpecializedTaskNodeClass(TSubclassOf<UK2Node_LatentGameplayTaskCall> NodeClass);
protected:
	static UE_API bool HasDedicatedNodeClass(TSubclassOf<UGameplayTask> TaskClass);

	virtual bool IsHandling(TSubclassOf<UGameplayTask> TaskClass) const { return true; }

private:
	static UE_API TArray<TWeakObjectPtr<UClass> > NodeClasses;
};

#undef UE_API
