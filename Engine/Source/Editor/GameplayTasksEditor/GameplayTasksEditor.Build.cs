// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class GameplayTasksEditor : ModuleRules
	{
        public GameplayTasksEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			OverridePackageType = PackageOverrideType.EngineDeveloper;

            PrivateDependencyModuleNames.AddRange(
                new string[]
				{
					// ... add private dependencies that you statically link with here ...
					"Core",
					"CoreUObject",
					"Engine",
					"GameplayTasks",
					"BlueprintGraph",
					"KismetCompiler",
					"UnrealEd",
				}
			);
		}
	}
}
