// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTasksEditorModule.h"

#include "CoreGlobals.h"
#include "Misc/AssertionMacros.h"

//////////////////////////////////////////////////////////////////////////
// FCharacterAIModule

class FGameplayTasksEditorModule : public IGameplayTasksEditorModule
{
public:
	virtual void StartupModule() override
	{
		check(GConfig);
	}

	virtual void ShutdownModule() override
	{
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FGameplayTasksEditorModule, GameplayTasksEditor);
