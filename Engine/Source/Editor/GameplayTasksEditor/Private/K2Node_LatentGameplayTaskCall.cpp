// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_LatentGameplayTaskCall.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/MemberReference.h"
#include "GameplayTask.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_CallArrayFunction.h"
#include "K2Node_CallFunction.h"
#include "K2Node_EnumLiteral.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet/KismetArrayLibrary.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_LatentGameplayTaskCall)


#define LOCTEXT_NAMESPACE "K2Node"

TArray<TWeakObjectPtr<UClass> > UK2Node_LatentGameplayTaskCall::NodeClasses;

UK2Node_LatentGameplayTaskCall::UK2Node_LatentGameplayTaskCall(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyActivateFunctionName = GET_FUNCTION_NAME_CHECKED(UGameplayTask, ReadyForActivation);
}

void UK2Node_LatentGameplayTaskCall::RegisterSpecializedTaskNodeClass(TSubclassOf<UK2Node_LatentGameplayTaskCall> NodeClass)
{
	if (NodeClass)
	{
		NodeClasses.AddUnique(*NodeClass);
	}
}

bool UK2Node_LatentGameplayTaskCall::HasDedicatedNodeClass(TSubclassOf<UGameplayTask> TaskClass) 
{
	for (const TWeakObjectPtr<UClass>& NodeClass : NodeClasses)
	{
		if (NodeClass.IsValid())
		{
			UK2Node_LatentGameplayTaskCall* NodeCDO = NodeClass->GetDefaultObject<UK2Node_LatentGameplayTaskCall>();
			if (NodeCDO && NodeCDO->IsHandling(TaskClass))
			{
				return true;
			}
		}
	}

	return false;
}

bool UK2Node_LatentGameplayTaskCall::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const
{
	return Super::CanCreateUnderSpecifiedSchema(DesiredSchema);
}

void UK2Node_LatentGameplayTaskCall::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	struct GetMenuActions_Utils
	{
		static void SetNodeFunc(UEdGraphNode* NewNode, bool /*bIsTemplateNode*/, TWeakObjectPtr<UFunction> FunctionPtr)
		{
			UK2Node_LatentGameplayTaskCall* AsyncTaskNode = CastChecked<UK2Node_LatentGameplayTaskCall>(NewNode);
			if (FunctionPtr.IsValid())
			{
				UFunction* Func = FunctionPtr.Get();
				FObjectProperty* ReturnProp = CastFieldChecked<FObjectProperty>(Func->GetReturnProperty());
						
				AsyncTaskNode->ProxyFactoryFunctionName = Func->GetFName();
				AsyncTaskNode->ProxyFactoryClass        = Func->GetOuterUClass();
				AsyncTaskNode->ProxyClass               = ReturnProp->PropertyClass;
			}
		}
	};

	UClass* NodeClass = GetClass();
	ActionRegistrar.RegisterClassFactoryActions<UGameplayTask>( FBlueprintActionDatabaseRegistrar::FMakeFuncSpawnerDelegate::CreateLambda([NodeClass](const UFunction* FactoryFunc)->UBlueprintNodeSpawner*
	{
		UBlueprintNodeSpawner* NodeSpawner = nullptr;
		
		UClass* FuncClass = FactoryFunc->GetOwnerClass();
		if (!UK2Node_LatentGameplayTaskCall::HasDedicatedNodeClass(FuncClass))
		{
			NodeSpawner = UBlueprintFunctionNodeSpawner::Create(FactoryFunc);
			check(NodeSpawner != nullptr);
			NodeSpawner->NodeClass = NodeClass;

			TWeakObjectPtr<UFunction> FunctionPtr = MakeWeakObjectPtr(const_cast<UFunction*>(FactoryFunc));
			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(GetMenuActions_Utils::SetNodeFunc, FunctionPtr);

		}
		return NodeSpawner;
	}) );
}

// -------------------------------------------------

struct FK2Node_LatentAbilityCallHelper
{
	static FName WorldContextPinName;
	static FName ClassPinName;
	static FName BeginSpawnFuncName;
	static FName FinishSpawnFuncName;
	static FName BeginSpawnArrayFuncName;
	static FName FinishSpawnArrayFuncName;
	static FName SpawnedActorPinName;
};

FName FK2Node_LatentAbilityCallHelper::WorldContextPinName(TEXT("WorldContextObject"));
FName FK2Node_LatentAbilityCallHelper::ClassPinName(TEXT("Class"));
FName FK2Node_LatentAbilityCallHelper::BeginSpawnFuncName(TEXT("BeginSpawningActor"));
FName FK2Node_LatentAbilityCallHelper::FinishSpawnFuncName(TEXT("FinishSpawningActor"));
FName FK2Node_LatentAbilityCallHelper::BeginSpawnArrayFuncName(TEXT("BeginSpawningActorArray"));
FName FK2Node_LatentAbilityCallHelper::FinishSpawnArrayFuncName(TEXT("FinishSpawningActorArray"));
FName FK2Node_LatentAbilityCallHelper::SpawnedActorPinName(TEXT("SpawnedActor"));

// -------------------------------------------------

void UK2Node_LatentGameplayTaskCall::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	AllocateDefaultPins();
	UClass* UseSpawnClass = GetClassToSpawn(&OldPins);
	if (UseSpawnClass != nullptr)
	{
		CreatePinsForClass(UseSpawnClass);
	}
	RestoreSplitPins(OldPins);
}

UEdGraphPin* UK2Node_LatentGameplayTaskCall::GetClassPin(const TArray<UEdGraphPin*>* InPinsToSearch /*= NULL*/) const
{
	const TArray<UEdGraphPin*>* PinsToSearch = InPinsToSearch ? InPinsToSearch : &Pins;

	UEdGraphPin* Pin = nullptr;
	for (UEdGraphPin* TestPin : *PinsToSearch)
	{
		if (TestPin && TestPin->PinName == FK2Node_LatentAbilityCallHelper::ClassPinName)
		{
			Pin = TestPin;
			break;
		}
	}
	check(Pin == nullptr || Pin->Direction == EGPD_Input);
	return Pin;
}

UClass* UK2Node_LatentGameplayTaskCall::GetClassToSpawn(const TArray<UEdGraphPin*>* InPinsToSearch) const
{
	UClass* UseSpawnClass = nullptr;
	const TArray<UEdGraphPin*>* PinsToSearch = InPinsToSearch ? InPinsToSearch : &Pins;

	UEdGraphPin* ClassPin = GetClassPin(PinsToSearch);
	if (ClassPin && ClassPin->DefaultObject != nullptr && ClassPin->LinkedTo.Num() == 0)
	{
		UseSpawnClass = CastChecked<UClass>(ClassPin->DefaultObject);
	}
	else if (ClassPin && (1 == ClassPin->LinkedTo.Num()))
	{
		UEdGraphPin* SourcePin = ClassPin->LinkedTo[0];
		UseSpawnClass = SourcePin ? Cast<UClass>(SourcePin->PinType.PinSubCategoryObject.Get()) : nullptr;
	}

	return UseSpawnClass;
}

void UK2Node_LatentGameplayTaskCall::CreatePinsForClass(UClass* InClass)
{
	check(InClass != nullptr);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	const UObject* const ClassDefaultObject = InClass->GetDefaultObject(false);

	SpawnParamPins.Reset();

	// Tasks can hide spawn parameters by doing meta = (HideSpawnParms="PropertyA,PropertyB")
	// (For example, hide Instigator in situations where instigator is not relevant to your task)
	
	TArray<FString> IgnorePropertyList;
	{
		UFunction* ProxyFunction = ProxyFactoryClass->FindFunctionByName(ProxyFactoryFunctionName);

		const FString& IgnorePropertyListStr = ProxyFunction->GetMetaData(FName(TEXT("HideSpawnParms")));
	
		if (!IgnorePropertyListStr.IsEmpty())
		{
			IgnorePropertyListStr.ParseIntoArray(IgnorePropertyList, TEXT(","), true);
		}
	}

	for (TFieldIterator<FProperty> PropertyIt(InClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		UClass* PropertyClass = Property->GetOwnerChecked<UClass>();
		const bool bIsDelegate = Property->IsA(FMulticastDelegateProperty::StaticClass());
		const bool bIsExposedToSpawn = UEdGraphSchema_K2::IsPropertyExposedOnSpawn(Property);
		const bool bIsSettableExternally = !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance);

		if (bIsExposedToSpawn &&
			!Property->HasAnyPropertyFlags(CPF_Parm) &&
			bIsSettableExternally &&
			Property->HasAllPropertyFlags(CPF_BlueprintVisible) &&
			!bIsDelegate && 
			!IgnorePropertyList.Contains(Property->GetName()) &&
			(FindPin(Property->GetFName()) == nullptr) )
		{


			UEdGraphPin* Pin = CreatePin(EGPD_Input, NAME_None, Property->GetFName());
			check(Pin);
			const bool bPinGood = K2Schema->ConvertPropertyToPinType(Property, /*out*/ Pin->PinType);
			SpawnParamPins.Add(Pin->PinName);

			if (ClassDefaultObject && K2Schema->PinDefaultValueIsEditable(*Pin))
			{
				FString DefaultValueAsString;
				const bool bDefaultValueSet = FBlueprintEditorUtils::PropertyValueToString(Property, reinterpret_cast<const uint8*>(ClassDefaultObject), DefaultValueAsString, this);
				check(bDefaultValueSet);
				K2Schema->SetPinAutogeneratedDefaultValue(Pin, DefaultValueAsString);
			}

			// Copy tooltip from the property.
			K2Schema->ConstructBasicPinTooltip(*Pin, Property->GetToolTipText(), Pin->PinToolTip);
		}
	}
}

void UK2Node_LatentGameplayTaskCall::PinDefaultValueChanged(UEdGraphPin* ChangedPin)
{
	if (ChangedPin->PinName == FK2Node_LatentAbilityCallHelper::ClassPinName)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		// Track removed pins so that we can reconnect it later if possible
		TArray<UEdGraphPin*> RemovedPins;

		// Orphan all pins related to archetype variables that have connections, otherwise just remove them
		for (const FName& OldPinReference : SpawnParamPins)
		{
			if(UEdGraphPin* OldPin = FindPin(OldPinReference))
			{
				if(OldPin->HasAnyConnections())
				{
					RemovedPins.Add(OldPin);
				}
				Pins.Remove(OldPin);
			}
		}
		
		SpawnParamPins.Reset();

		UClass* UseSpawnClass = GetClassToSpawn();
		if (UseSpawnClass != nullptr)
		{
			CreatePinsForClass(UseSpawnClass);
		}

		RewireOldPinsToNewPins(/* InOldPins = */ RemovedPins, /* InNewPins = */ Pins, /* NewPinToOldPin = */ nullptr);

		// Refresh the UI for the graph so the pin changes show up
		UEdGraph* Graph = GetGraph();
		Graph->NotifyNodeChanged(this);

		// Mark dirty
		FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprint());
	}
}

UEdGraphPin* UK2Node_LatentGameplayTaskCall::GetResultPin() const
{
	UEdGraphPin* Pin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}

bool UK2Node_LatentGameplayTaskCall::IsSpawnVarPin(UEdGraphPin* Pin)
{
	return (Pin->Direction == EEdGraphPinDirection::EGPD_Input &&
			Pin->PinName != UEdGraphSchema_K2::PN_Execute &&
			Pin->PinName != UEdGraphSchema_K2::PN_Then &&
			Pin->PinName != UEdGraphSchema_K2::PN_ReturnValue &&
			Pin->PinName != FK2Node_LatentAbilityCallHelper::ClassPinName &&
			Pin->PinName != FK2Node_LatentAbilityCallHelper::WorldContextPinName);

}

bool UK2Node_LatentGameplayTaskCall::ValidateActorSpawning(class FKismetCompilerContext& CompilerContext, bool bGenerateErrors)
{
	FName ProxyPrespawnFunctionName = FK2Node_LatentAbilityCallHelper::BeginSpawnFuncName;
	UFunction* PreSpawnFunction = ProxyFactoryClass ? ProxyFactoryClass->FindFunctionByName(ProxyPrespawnFunctionName) : nullptr;

	FName ProxyPostpawnFunctionName = FK2Node_LatentAbilityCallHelper::FinishSpawnFuncName;
	UFunction* PostSpawnFunction = ProxyFactoryClass ? ProxyFactoryClass->FindFunctionByName(ProxyPostpawnFunctionName) : nullptr;

	FName ProxyPrespawnArrayFunctionName = FK2Node_LatentAbilityCallHelper::BeginSpawnArrayFuncName;
	UFunction* PreSpawnArrayFunction = ProxyFactoryClass ? ProxyFactoryClass->FindFunctionByName(ProxyPrespawnArrayFunctionName) : nullptr;

	FName ProxyPostpawnArrayFunctionName = FK2Node_LatentAbilityCallHelper::FinishSpawnArrayFuncName;
	UFunction* PostSpawnArrayFunction = ProxyFactoryClass ? ProxyFactoryClass->FindFunctionByName(ProxyPostpawnArrayFunctionName) : nullptr;

	bool HasClassParameter = GetClassPin() != nullptr;
	bool HasPreSpawnFunc = PreSpawnFunction != nullptr;
	bool HasPostSpawnFunc = PostSpawnFunction != nullptr;
	bool HasPreSpawnArrayFunc = PreSpawnArrayFunction != nullptr;
	bool HasPostSpawnArrayFunc = PostSpawnArrayFunction != nullptr;

	if (HasClassParameter || HasPreSpawnFunc || HasPostSpawnFunc)
	{
		// They are trying to use ActorSpawning. If any of the above are NOT true, then we have a problem
		if (!HasClassParameter)
		{
			if (bGenerateErrors)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingClassParameter", "UK2Node_LatentGameplayTaskCall: Attempting to use ActorSpawning but Proxy Factory Function missing a Class parameter. @@").ToString(), this);
			}
			return false;
		}
		if (!HasPreSpawnFunc)
		{
			if (bGenerateErrors)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingBeginSpawningFunc", "UK2Node_LatentGameplayTaskCall: Attempting to use ActorSpawning but Missing a BeginSpawningActor function. @@").ToString(), this);
			}
			return false;
		}
		if (!HasPostSpawnFunc)
		{
			if (bGenerateErrors)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingFinishSpawningFunc", "UK2Node_LatentGameplayTaskCall: Attempting to use ActorSpawning but Missing a FinishSpawningActor function. @@").ToString(), this);
			}
			return false;
		}
		if ((HasPreSpawnArrayFunc || HasPostSpawnArrayFunc))
		{
			if (bGenerateErrors)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("SpawnFuncAmbiguous", "UK2Node_LatentGameplayTaskCall: Both ActorSpawning and ActorArraySpawning are at least partially implemented. These are mutually exclusive. @@").ToString(), this);
			}
			return false;
		}
	}
	
	return true;
}

bool UK2Node_LatentGameplayTaskCall::ValidateActorArraySpawning(class FKismetCompilerContext& CompilerContext, bool bGenerateErrors)
{
	FName ProxyPrespawnFunctionName = FK2Node_LatentAbilityCallHelper::BeginSpawnFuncName;
	UFunction* PreSpawnFunction = ProxyFactoryClass ? ProxyFactoryClass->FindFunctionByName(ProxyPrespawnFunctionName) : nullptr;

	FName ProxyPostpawnFunctionName = FK2Node_LatentAbilityCallHelper::FinishSpawnFuncName;
	UFunction* PostSpawnFunction = ProxyFactoryClass ? ProxyFactoryClass->FindFunctionByName(ProxyPostpawnFunctionName) : nullptr;

	FName ProxyPrespawnArrayFunctionName = FK2Node_LatentAbilityCallHelper::BeginSpawnArrayFuncName;
	UFunction* PreSpawnArrayFunction = ProxyFactoryClass ? ProxyFactoryClass->FindFunctionByName(ProxyPrespawnArrayFunctionName) : nullptr;

	FName ProxyPostpawnArrayFunctionName = FK2Node_LatentAbilityCallHelper::FinishSpawnArrayFuncName;
	UFunction* PostSpawnArrayFunction = ProxyFactoryClass ? ProxyFactoryClass->FindFunctionByName(ProxyPostpawnArrayFunctionName) : nullptr;

	bool HasClassParameter = GetClassToSpawn() != nullptr;
	bool HasPreSpawnFunc = PreSpawnFunction != nullptr;
	bool HasPostSpawnFunc = PostSpawnFunction != nullptr;
	bool HasPreSpawnArrayFunc = PreSpawnArrayFunction != nullptr;
	bool HasPostSpawnArrayFunc = PostSpawnArrayFunction != nullptr;

	if (HasClassParameter || HasPreSpawnFunc || HasPostSpawnFunc || HasPreSpawnArrayFunc || HasPostSpawnArrayFunc)
	{
		// They are trying to use ActorSpawning. If any of the above are NOT true, then we have a problem
		if (!HasClassParameter)
		{
			if (bGenerateErrors)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingClassParameter", "UK2Node_LatentGameplayTaskCall: Attempting to use ActorSpawning but Proxy Factory Function missing a Class parameter. @@").ToString(), this);
			}
			return false;
		}
		if (!HasPreSpawnArrayFunc)
		{
			if (bGenerateErrors)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingBeginSpawningArrayFunc", "UK2Node_LatentGameplayTaskCall: Attempting to use ActorArraySpawning but Missing a BeginSpawningActorArray function. @@").ToString(), this);
			}
			return false;
		}
		if (!HasPostSpawnArrayFunc)
		{
			if (bGenerateErrors)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingFinishSpawningArrayFunc", "UK2Node_LatentGameplayTaskCall: Attempting to use ActorArraySpawning but Missing a FinishSpawningActorArray function. @@").ToString(), this);
			}
			return false;
		}
		if (HasPreSpawnFunc || HasPostSpawnFunc)
		{
			if (bGenerateErrors)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("SpawnFuncAmbiguous", "UK2Node_LatentGameplayTaskCall: Both ActorSpawning and ActorArraySpawning are at least partially implemented. These are mutually exclusive. @@").ToString(), this);
			}
			return false;
		}
	}

	return true;
}

bool UK2Node_LatentGameplayTaskCall::ConnectSpawnProperties(UClass* ClassToSpawn, const UEdGraphSchema_K2* Schema, class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UEdGraphPin*& LastThenPin, UEdGraphPin* SpawnedActorReturnPin)
{
	bool bIsErrorFree = true;
	for (const FName& OldPinReference : SpawnParamPins)
	{
		UEdGraphPin* SpawnVarPin = FindPin(OldPinReference);
		if (!SpawnVarPin)
		{
			continue;
		}

		const bool bHasDefaultValue = !SpawnVarPin->DefaultValue.IsEmpty() || !SpawnVarPin->DefaultTextValue.IsEmpty() || SpawnVarPin->DefaultObject;
		if (SpawnVarPin->LinkedTo.Num() > 0 || bHasDefaultValue)
		{
			if (SpawnVarPin->LinkedTo.Num() == 0)
			{
				FProperty* Property = FindFProperty<FProperty>(ClassToSpawn, SpawnVarPin->PinName);
				// NULL property indicates that this pin was part of the original node, not the 
				// class we're assigning to:
				if (!Property)
				{
					continue;
				}

				// This is sloppy, we should be comparing to defaults much later in the compile process:
				if (ClassToSpawn->GetDefaultObject(false) != nullptr)
				{
					// We don't want to generate an assignment node unless the default value 
					// differs from the value in the CDO:
					FString DefaultValueAsString;
					FBlueprintEditorUtils::PropertyValueToString(Property, (uint8*)ClassToSpawn->GetDefaultObject(false), DefaultValueAsString, this);
					if (DefaultValueAsString == SpawnVarPin->DefaultValue)
					{
						continue;
					}
				}
			}


			UFunction* SetByNameFunction = Schema->FindSetVariableByNameFunction(SpawnVarPin->PinType);
			if (SetByNameFunction)
			{
				UK2Node_CallFunction* SetVarNode = nullptr;
				if (SpawnVarPin->PinType.IsArray())
				{
					SetVarNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallArrayFunction>(this, SourceGraph);
				}
				else
				{
					SetVarNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
				}
				SetVarNode->SetFromFunction(SetByNameFunction);
				SetVarNode->AllocateDefaultPins();

				// Connect this node into the exec chain
				bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, SetVarNode->GetExecPin());
				LastThenPin = SetVarNode->GetThenPin();

				static const FName ObjectParamName(TEXT("Object"));
				static const FName ValueParamName(TEXT("Value"));
				static const FName PropertyNameParamName(TEXT("PropertyName"));

				// Connect the new actor to the 'object' pin
				UEdGraphPin* ObjectPin = SetVarNode->FindPinChecked(ObjectParamName);
				SpawnedActorReturnPin->MakeLinkTo(ObjectPin);

				// Fill in literal for 'property name' pin - name of pin is property name
				UEdGraphPin* PropertyNamePin = SetVarNode->FindPinChecked(PropertyNameParamName);
				PropertyNamePin->DefaultValue = SpawnVarPin->PinName.ToString();

				UEdGraphPin* ValuePin = SetVarNode->FindPinChecked(ValueParamName);
				if (SpawnVarPin->LinkedTo.Num() == 0 &&
					SpawnVarPin->DefaultValue != FString() &&
					SpawnVarPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Byte &&
					SpawnVarPin->PinType.PinSubCategoryObject.IsValid() &&
					SpawnVarPin->PinType.PinSubCategoryObject->IsA<UEnum>())
				{
					// Pin is an enum, we need to alias the enum value to an int:
					UK2Node_EnumLiteral* EnumLiteralNode = CompilerContext.SpawnIntermediateNode<UK2Node_EnumLiteral>(this, SourceGraph);
					EnumLiteralNode->Enum = CastChecked<UEnum>(SpawnVarPin->PinType.PinSubCategoryObject.Get());
					EnumLiteralNode->AllocateDefaultPins();
					EnumLiteralNode->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue)->MakeLinkTo(ValuePin);

					UEdGraphPin* InPin = EnumLiteralNode->FindPinChecked(UK2Node_EnumLiteral::GetEnumInputPinName());
					InPin->DefaultValue = SpawnVarPin->DefaultValue;
				}
				else
				{
					// For non-array struct pins that are not linked, transfer the pin type so that the node will expand an auto-ref that will assign the value by-ref.
					if (SpawnVarPin->PinType.IsArray() == false && SpawnVarPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && SpawnVarPin->LinkedTo.Num() == 0)
					{
						ValuePin->PinType.PinCategory = SpawnVarPin->PinType.PinCategory;
						ValuePin->PinType.PinSubCategory = SpawnVarPin->PinType.PinSubCategory;
						ValuePin->PinType.PinSubCategoryObject = SpawnVarPin->PinType.PinSubCategoryObject;
						CompilerContext.MovePinLinksToIntermediate(*SpawnVarPin, *ValuePin);
					}
					else
					{
						// Move connection from the variable pin on the spawn node to the 'value' pin
						CompilerContext.MovePinLinksToIntermediate(*SpawnVarPin, *ValuePin);
						SetVarNode->PinConnectionListChanged(ValuePin);
					}
				}
			}
		}
	}
	return bIsErrorFree;
}

/**
 *	This is essentially a mix of K2Node_BaseAsyncTask::ExpandNode and K2Node_SpawnActorFromClass::ExpandNode.
 *	Several things are going on here:
 *		-Factory call to create proxy object (K2Node_BaseAsyncTask)
 *		-Task return delegates are created and hooked up (K2Node_BaseAsyncTask)
 *		-A BeginSpawn function is called on proxyu object (similiar to K2Node_SpawnActorFromClass)
 *		-BeginSpawn can choose to spawn or not spawn an actor (and return it)
 *			-If spawned:
 *				-SetVars are run on the newly spawned object (set expose on spawn variables - K2Node_SpawnActorFromClass)
 *				-FinishSpawn is called on the proxy object
 *				
 *				
 *	Also, a K2Node_SpawnActorFromClass could not be used directly here, since we want the proxy object to implement its own
 *	BeginSpawn/FinishSpawn function (custom game logic will often be performed in the native implementation). K2Node_SpawnActorFromClass also
 *	requires a SpawnTransform be wired into it, and in most ability task cases, the spawn transform is implied or not necessary.
 *	
 *	
 */
void UK2Node_LatentGameplayTaskCall::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	bool bValidatedActorSpawn = ValidateActorSpawning(CompilerContext, false);
	bool bValidatedActorArraySpawn = ValidateActorArraySpawning(CompilerContext, false);

	UEdGraphPin* ClassPin = GetClassPin();
	if (ClassPin == nullptr)
	{
		// Nothing special about this task, just call super
		Super::ExpandNode(CompilerContext, SourceGraph);
		return;
	}

	UK2Node::ExpandNode(CompilerContext, SourceGraph);

	if (!bValidatedActorSpawn && !bValidatedActorArraySpawn)
	{
		ValidateActorSpawning(CompilerContext, true);
		ValidateActorArraySpawning(CompilerContext, true);
	}

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	check(SourceGraph && Schema);
	bool bIsErrorFree = true;


	// ------------------------------------------------------------------------------------------
	// CREATE A CALL TO FACTORY THE PROXY OBJECT
	// ------------------------------------------------------------------------------------------
	UK2Node_CallFunction* const CallCreateProxyObjectNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallCreateProxyObjectNode->FunctionReference.SetExternalMember(ProxyFactoryFunctionName, ProxyFactoryClass);
	CallCreateProxyObjectNode->AllocateDefaultPins();
	bIsErrorFree &= CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UEdGraphSchema_K2::PN_Execute), *CallCreateProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute)).CanSafeConnect();
	for (UEdGraphPin* CurrentPin : Pins)
	{
		if (FBaseAsyncTaskHelper::ValidDataPin(CurrentPin, EGPD_Input))
		{
			UEdGraphPin* DestPin = CallCreateProxyObjectNode->FindPin(CurrentPin->PinName); // match function inputs, to pass data to function from CallFunction node

			// NEW: if no DestPin, assume it is a Class Spawn PRoperty - not an error
			if (DestPin)
			{
				bIsErrorFree &= CompilerContext.CopyPinLinksToIntermediate(*CurrentPin, *DestPin).CanSafeConnect();
			}
		}
	}
	
	bIsErrorFree &= ExpandDefaultToSelfPin(CompilerContext, SourceGraph, CallCreateProxyObjectNode);

	// Expose Async Task Proxy object
	UEdGraphPin* const ProxyObjectPin = CallCreateProxyObjectNode->GetReturnValuePin();
	check(ProxyObjectPin);
	UEdGraphPin* OutputAsyncTaskProxy = FindPinChecked(FBaseAsyncTaskHelper::GetAsyncTaskProxyName());
	bIsErrorFree &= CompilerContext.MovePinLinksToIntermediate(*OutputAsyncTaskProxy, *ProxyObjectPin).CanSafeConnect();

	// ------------------------------------------------------------------------------------------
	// GATHER OUTPUT PARAMETERS AND PAIR THEM WITH LOCAL VARIABLES
	// ------------------------------------------------------------------------------------------
	TArray<FBaseAsyncTaskHelper::FOutputPinAndLocalVariable> VariableOutputs;
	for (UEdGraphPin* CurrentPin : Pins)
	{
		if ((OutputAsyncTaskProxy != CurrentPin) && FBaseAsyncTaskHelper::ValidDataPin(CurrentPin, EGPD_Output))
		{
			const FEdGraphPinType& PinType = CurrentPin->PinType;
			UK2Node_TemporaryVariable* TempVarOutput = CompilerContext.SpawnInternalVariable(
				this, PinType.PinCategory, PinType.PinSubCategory, PinType.PinSubCategoryObject.Get(), PinType.ContainerType, PinType.PinValueType);
			bIsErrorFree &= TempVarOutput->GetVariablePin() && CompilerContext.MovePinLinksToIntermediate(*CurrentPin, *TempVarOutput->GetVariablePin()).CanSafeConnect();
			VariableOutputs.Add(FBaseAsyncTaskHelper::FOutputPinAndLocalVariable(CurrentPin, TempVarOutput));
		}
	}

	// ------------------------------------------------------------------------------------------
	// FOR EACH DELEGATE DEFINE EVENT, CONNECT IT TO DELEGATE AND IMPLEMENT A CHAIN OF ASSIGMENTS
	// ------------------------------------------------------------------------------------------
	UEdGraphPin* LastThenPin = CallCreateProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);

	UK2Node_CallFunction* IsValidFuncNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	const FName IsValidFuncName = GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, IsValid);
	IsValidFuncNode->FunctionReference.SetExternalMember(IsValidFuncName, UKismetSystemLibrary::StaticClass());
	IsValidFuncNode->AllocateDefaultPins();
	UEdGraphPin* IsValidInputPin = IsValidFuncNode->FindPinChecked(TEXT("Object"));

	bIsErrorFree &= Schema->TryCreateConnection(ProxyObjectPin, IsValidInputPin);

	UK2Node_IfThenElse* ValidateProxyNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
	ValidateProxyNode->AllocateDefaultPins();
	bIsErrorFree &= Schema->TryCreateConnection(IsValidFuncNode->GetReturnValuePin(), ValidateProxyNode->GetConditionPin());

	bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, ValidateProxyNode->GetExecPin());
	LastThenPin = ValidateProxyNode->GetThenPin();

	for (TFieldIterator<FMulticastDelegateProperty> PropertyIt(ProxyClass); PropertyIt && bIsErrorFree; ++PropertyIt)
	{
		UEdGraphPin* LastActivatedThenPin = nullptr;
		bIsErrorFree &= FBaseAsyncTaskHelper::HandleDelegateImplementation(*PropertyIt, VariableOutputs, ProxyObjectPin, LastThenPin, LastActivatedThenPin, this, SourceGraph, CompilerContext);
	}

	if (CallCreateProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Then) == LastThenPin)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingDelegateProperties", "BaseAsyncTask: Proxy has no delegates defined. @@").ToString(), this);
		return;
	}


	// ------------------------------------------------------------------------------------------
	// NEW: CREATE A CALL TO THE PRESPAWN FUNCTION, IF IT RETURNS TRUE, THEN WE WILL SPAWN THE NEW ACTOR
	// ------------------------------------------------------------------------------------------

	FName ProxyPrespawnFunctionName = bValidatedActorArraySpawn ? FK2Node_LatentAbilityCallHelper::BeginSpawnArrayFuncName : FK2Node_LatentAbilityCallHelper::BeginSpawnFuncName;
	UFunction* PreSpawnFunction = ProxyFactoryClass ? ProxyFactoryClass->FindFunctionByName(ProxyPrespawnFunctionName) : nullptr;

	FName ProxyPostpawnFunctionName = bValidatedActorArraySpawn ? FK2Node_LatentAbilityCallHelper::FinishSpawnArrayFuncName : FK2Node_LatentAbilityCallHelper::FinishSpawnFuncName;
	UFunction* PostSpawnFunction = ProxyFactoryClass ? ProxyFactoryClass->FindFunctionByName(ProxyPostpawnFunctionName) : nullptr;

	if (PreSpawnFunction == nullptr)
	{
		if (bValidatedActorArraySpawn)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MissingBeginSpawningActorArrayFunction", "AbilityTask: Proxy is missing BeginSpawningActorArray native function. @@").ToString(), this);
		}
		else
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MissingBeginSpawningActorFunction", "AbilityTask: Proxy is missing BeginSpawningActor native function. @@").ToString(), this);
		}
		return;
	}

	if (PostSpawnFunction == nullptr)
	{
		if (bValidatedActorArraySpawn)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MissingFinishSpawningActorArrayFunction", "AbilityTask: Proxy is missing FinishSpawningActorArray native function. @@").ToString(), this);
		}
		else
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MissingFinishSpawningActorFunction", "AbilityTask: Proxy is missing FinishSpawningActor native function. @@").ToString(), this);
		}
		return;
	}


	UK2Node_CallFunction* const CallPrespawnProxyObjectNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallPrespawnProxyObjectNode->FunctionReference.SetExternalMember(ProxyPrespawnFunctionName, ProxyClass);
	CallPrespawnProxyObjectNode->AllocateDefaultPins();

	// Hook up the self connection
	UEdGraphPin* PrespawnCallSelfPin = Schema->FindSelfPin(*CallPrespawnProxyObjectNode, EGPD_Input);
	check(PrespawnCallSelfPin);

	bIsErrorFree &= Schema->TryCreateConnection(ProxyObjectPin, PrespawnCallSelfPin);

	// Hook up input parameters to PreSpawn
	for (UEdGraphPin* CurrentPin : Pins)
	{
		if (FBaseAsyncTaskHelper::ValidDataPin(CurrentPin, EGPD_Input))
		{
			UEdGraphPin* DestPin = CallPrespawnProxyObjectNode->FindPin(CurrentPin->PinName);
			if (DestPin)
			{
				bIsErrorFree &= CompilerContext.CopyPinLinksToIntermediate(*CurrentPin, *DestPin).CanSafeConnect();
			}
		}
	}		

	// Hook the activate node up in the exec chain
	UEdGraphPin* PrespawnExecPin = CallPrespawnProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute);
	UEdGraphPin* PrespawnThenPin = CallPrespawnProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);
	UEdGraphPin* PrespawnReturnPin = CallPrespawnProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	UEdGraphPin* SpawnedActorReturnPin = CallPrespawnProxyObjectNode->FindPinChecked(FK2Node_LatentAbilityCallHelper::SpawnedActorPinName);

	bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, PrespawnExecPin);

	LastThenPin = PrespawnThenPin;

	// -------------------------------------------
	// Branch based on return value of Prespawn
	// -------------------------------------------
		
	UK2Node_IfThenElse* BranchNode = SourceGraph->CreateIntermediateNode<UK2Node_IfThenElse>();
	BranchNode->AllocateDefaultPins();
	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(BranchNode, this);

	// Link return value of prespawn with the branch condtional
	bIsErrorFree &= Schema->TryCreateConnection(PrespawnReturnPin, BranchNode->GetConditionPin());

	// Link our Prespawn call to the branch node
	bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, BranchNode->GetExecPin());

	UEdGraphPin* BranchElsePin = BranchNode->GetElsePin();

	LastThenPin = BranchNode->GetThenPin();

	UClass* ClassToSpawn = GetClassToSpawn();
	if (bValidatedActorArraySpawn && ClassToSpawn)
	{
		//Branch for main loop control
		UK2Node_IfThenElse* Branch = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
		Branch->AllocateDefaultPins();

		//Create int Iterator
		UK2Node_TemporaryVariable* IteratorVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
		IteratorVar->VariableType.PinCategory = UEdGraphSchema_K2::PC_Int;
		IteratorVar->AllocateDefaultPins();

		//Iterator assignment (initialization to zero)
		UK2Node_AssignmentStatement* IteratorInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
		IteratorInitialize->AllocateDefaultPins();
		IteratorInitialize->GetValuePin()->DefaultValue = TEXT("0");

		//Iterator assignment (incrementing)
		UK2Node_AssignmentStatement* IteratorAssign = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
		IteratorAssign->AllocateDefaultPins();

		//Increment iterator command
		UK2Node_CallFunction* Increment = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		Increment->SetFromFunction(UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Add_IntInt)));
		Increment->AllocateDefaultPins();
		Increment->FindPinChecked(TEXT("B"))->DefaultValue = TEXT("1");

		//Array length
		UK2Node_CallArrayFunction* ArrayLength = CompilerContext.SpawnIntermediateNode<UK2Node_CallArrayFunction>(this, SourceGraph);
		ArrayLength->SetFromFunction(UKismetArrayLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetArrayLibrary, Array_Length)));
		ArrayLength->AllocateDefaultPins();

		//Array element retrieval
		UK2Node_CallArrayFunction* GetElement = CompilerContext.SpawnIntermediateNode<UK2Node_CallArrayFunction>(this, SourceGraph);
		GetElement->SetFromFunction(UKismetArrayLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetArrayLibrary, Array_Get)));
		GetElement->AllocateDefaultPins();

		//Check node for iterator versus array length
		UK2Node_CallFunction* Condition = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		Condition->SetFromFunction(UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Less_IntInt)));
		Condition->AllocateDefaultPins();

		//Connections to set up the loop
		bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, IteratorInitialize->GetExecPin());
		bIsErrorFree &= Schema->TryCreateConnection(IteratorVar->GetVariablePin(), IteratorInitialize->GetVariablePin());
		bIsErrorFree &= Schema->TryCreateConnection(IteratorInitialize->GetThenPin(), Branch->GetExecPin());
		bIsErrorFree &= Schema->TryCreateConnection(SpawnedActorReturnPin, ArrayLength->GetTargetArrayPin());
		bIsErrorFree &= Schema->TryCreateConnection(Condition->GetReturnValuePin(), Branch->GetConditionPin());
		bIsErrorFree &= Schema->TryCreateConnection(IteratorVar->GetVariablePin(), Condition->FindPinChecked(TEXT("A")));
		bIsErrorFree &= Schema->TryCreateConnection(ArrayLength->FindPin(UEdGraphSchema_K2::PN_ReturnValue), Condition->FindPinChecked(TEXT("B")));

		//Connections to establish loop iteration
		bIsErrorFree &= Schema->TryCreateConnection(IteratorVar->GetVariablePin(), Increment->FindPinChecked(TEXT("A")));
		bIsErrorFree &= Schema->TryCreateConnection(IteratorVar->GetVariablePin(), IteratorAssign->GetVariablePin());
		bIsErrorFree &= Schema->TryCreateConnection(Increment->GetReturnValuePin(), IteratorAssign->GetValuePin());
		bIsErrorFree &= Schema->TryCreateConnection(IteratorAssign->GetThenPin(), Branch->GetExecPin());

		//This is the inner loop
		LastThenPin = Branch->GetThenPin();		//Connect the loop branch to the spawn-assignment code block
		bIsErrorFree &= Schema->TryCreateConnection(SpawnedActorReturnPin, GetElement->GetTargetArrayPin());
		bIsErrorFree &= Schema->TryCreateConnection(IteratorVar->GetVariablePin(), GetElement->FindPinChecked(UEdGraphSchema_K2::PN_Index));
		bIsErrorFree &= ConnectSpawnProperties(ClassToSpawn, Schema, CompilerContext, SourceGraph, LastThenPin, GetElement->FindPinChecked(UEdGraphSchema_K2::PN_Item));		//Last argument is the array element
		bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, IteratorAssign->GetExecPin());		//Connect the spawn-assignment code block to the iterator increment
		
		//Finish by providing the proper path out
		LastThenPin = Branch->GetElsePin();
	}

	// -------------------------------------------
	// Set spawn variables
	//  Borrowed heavily from FKismetCompilerUtilities::GenerateAssignmentNodes
	// -------------------------------------------
	
	if (bValidatedActorSpawn && ClassToSpawn)
	{
		bIsErrorFree &= ConnectSpawnProperties(ClassToSpawn, Schema, CompilerContext, SourceGraph, LastThenPin, SpawnedActorReturnPin);
	}
	
	// -------------------------------------------
	// Call FinishSpawning
	// -------------------------------------------

	UK2Node_CallFunction* const CallPostSpawnnProxyObjectNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallPostSpawnnProxyObjectNode->FunctionReference.SetExternalMember(ProxyPostpawnFunctionName, ProxyClass);
	CallPostSpawnnProxyObjectNode->AllocateDefaultPins();

	// Hook up the self connection
	UEdGraphPin* PostspawnCallSelfPin = Schema->FindSelfPin(*CallPostSpawnnProxyObjectNode, EGPD_Input);
	check(PostspawnCallSelfPin);

	bIsErrorFree &= Schema->TryCreateConnection(ProxyObjectPin, PostspawnCallSelfPin);

	// Link our Postspawn call in
	bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, CallPostSpawnnProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute));

	// Hook up any other input parameters to PostSpawn
	for (UEdGraphPin* CurrentPin : Pins)
	{
		if (FBaseAsyncTaskHelper::ValidDataPin(CurrentPin, EGPD_Input))
		{
			UEdGraphPin* DestPin = CallPostSpawnnProxyObjectNode->FindPin(CurrentPin->PinName);
			if (DestPin)
			{
				bIsErrorFree &= CompilerContext.CopyPinLinksToIntermediate(*CurrentPin, *DestPin).CanSafeConnect();
			}
		}
	}


	UEdGraphPin* InSpawnedActorPin = CallPostSpawnnProxyObjectNode->FindPin(TEXT("SpawnedActor"));
	if (InSpawnedActorPin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingSpawnedActorInputPin", "AbilityTask: Proxy is missing SpawnedActor input pin in FinishSpawningActor. @@").ToString(), this);
		return;
	}

	bIsErrorFree &= Schema->TryCreateConnection(SpawnedActorReturnPin, InSpawnedActorPin);

	LastThenPin = CallPostSpawnnProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);

	// --------------------------------------------------------------------------------------
	// Create a call to activate the proxy object if necessary
	// --------------------------------------------------------------------------------------

	UK2Node_IfThenElse* ProxyActivateValidateProxyNode = nullptr;

	if (ProxyActivateFunctionName != NAME_None)
	{
		// Validate the proxy object is still valid. Its possible the task ends while calling FinishSpawning, in which case we don't need to call the ProxyActivateFunction.		
		UK2Node_CallFunction* ProxyActivateIsValidFuncNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		ProxyActivateIsValidFuncNode->FunctionReference.SetExternalMember(IsValidFuncName, UKismetSystemLibrary::StaticClass());
		ProxyActivateIsValidFuncNode->AllocateDefaultPins();
		UEdGraphPin* ProxyActivateIsValidInputPin = ProxyActivateIsValidFuncNode->FindPinChecked(TEXT("Object"));

		bIsErrorFree &= Schema->TryCreateConnection(ProxyObjectPin, ProxyActivateIsValidInputPin);

		ProxyActivateValidateProxyNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
		ProxyActivateValidateProxyNode->AllocateDefaultPins();
		bIsErrorFree &= Schema->TryCreateConnection(ProxyActivateIsValidFuncNode->GetReturnValuePin(), ProxyActivateValidateProxyNode->GetConditionPin());

		bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, ProxyActivateValidateProxyNode->GetExecPin());
		LastThenPin = ProxyActivateValidateProxyNode->GetThenPin();

		// Actually call the Activate function
		UK2Node_CallFunction* const CallActivateProxyObjectNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CallActivateProxyObjectNode->FunctionReference.SetExternalMember(ProxyActivateFunctionName, ProxyClass);
		CallActivateProxyObjectNode->AllocateDefaultPins();

		// Hook up the self connection
		UEdGraphPin* ActivateCallSelfPin = Schema->FindSelfPin(*CallActivateProxyObjectNode, EGPD_Input);
		check(ActivateCallSelfPin);

		bIsErrorFree &= Schema->TryCreateConnection(ProxyObjectPin, ActivateCallSelfPin);

		// Hook the activate node up in the exec chain
		UEdGraphPin* ActivateExecPin = CallActivateProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute);
		UEdGraphPin* ActivateThenPin = CallActivateProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);

		bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, ActivateExecPin);

		LastThenPin = ActivateThenPin;
	}

	// --------------------------------------------------------------------------------------
	// Move the connections from the original node then pin to the last internal then pin
	// --------------------------------------------------------------------------------------

	bIsErrorFree &= CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UEdGraphSchema_K2::PN_Then), *LastThenPin).CanSafeConnect();
	bIsErrorFree &= CompilerContext.CopyPinLinksToIntermediate(*LastThenPin, *BranchElsePin).CanSafeConnect();
	bIsErrorFree &= CompilerContext.CopyPinLinksToIntermediate(*LastThenPin, *ValidateProxyNode->GetElsePin()).CanSafeConnect();
	if (ProxyActivateValidateProxyNode)
	{
		bIsErrorFree &= CompilerContext.CopyPinLinksToIntermediate(*LastThenPin, *ProxyActivateValidateProxyNode->GetElsePin()).CanSafeConnect();
	}
	
	if (!bIsErrorFree)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InternalConnectionError", "BaseAsyncTask: Internal connection error. @@").ToString(), this);
	}

	// Make sure we caught everything
	BreakAllNodeLinks();
}


#undef LOCTEXT_NAMESPACE
